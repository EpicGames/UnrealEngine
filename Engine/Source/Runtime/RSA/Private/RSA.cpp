// Copyright Epic Games, Inc. All Rights Reserved.

#include "RSA.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

static FCriticalSection GLock;

IEngineCrypto* GetEngineCrypto()
{
	static TArray<IEngineCrypto*> Features = IModularFeatures::Get().GetModularFeatureImplementations<IEngineCrypto>(IEngineCrypto::GetFeatureName());
	checkf(Features.Num() > 0, TEXT("RSA functionality was used but no modular feature was registered to provide it. Please make sure your project has the PlatformCrypto plugin enabled!"));
	return Features[0];
}

FRSAKeyHandle FRSA::CreateKey(const TArray<uint8>& InPublicExponent, const TArray<uint8>& InPrivateExponent, const TArray<uint8>& InModulus)
{
	if (IEngineCrypto* EngineCrypto = GetEngineCrypto())
	{
		return EngineCrypto->CreateRSAKey(InPublicExponent, InPrivateExponent, InModulus);
	}

	return nullptr;
}

void FRSA::DestroyKey(const FRSAKeyHandle InKey)
{
	if (IEngineCrypto* EngineCrypto = GetEngineCrypto())
	{
		EngineCrypto->DestroyRSAKey(InKey);
	}
}

int32 FRSA::GetKeySize(const FRSAKeyHandle InKey)
{
	if (IEngineCrypto* EngineCrypto = GetEngineCrypto())
	{
		return EngineCrypto->GetKeySize(InKey);
	}

	return 0;
}

int32 FRSA::GetMaxDataSize(const FRSAKeyHandle InKey)
{
	if (IEngineCrypto* EngineCrypto = GetEngineCrypto())
	{
		return EngineCrypto->GetMaxDataSize(InKey);
	}

	return 0;
}

int32 FRSA::EncryptPublic(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, const FRSAKeyHandle InKey)
{
	if (IEngineCrypto* EngineCrypto = GetEngineCrypto())
	{
		return EngineCrypto->EncryptPublic(InSource, OutDestination, InKey);
	}

	return -1;
}

int32 FRSA::EncryptPrivate(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, const FRSAKeyHandle InKey)
{
	if (IEngineCrypto* EngineCrypto = GetEngineCrypto())
	{
		return EngineCrypto->EncryptPrivate(InSource, OutDestination, InKey);
	}

	return -1;
}

int32 FRSA::DecryptPublic(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, const FRSAKeyHandle InKey)
{
	if (IEngineCrypto* EngineCrypto = GetEngineCrypto())
	{
		return EngineCrypto->DecryptPublic(InSource, OutDestination, InKey);
	}

	return -1;
}

int32 FRSA::DecryptPrivate(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, const FRSAKeyHandle InKey)
{
	if (IEngineCrypto* EngineCrypto = GetEngineCrypto())
	{
		return EngineCrypto->DecryptPrivate(InSource, OutDestination, InKey);
	}

	return -1;
}

IMPLEMENT_MODULE(FDefaultModuleImpl, RSA);