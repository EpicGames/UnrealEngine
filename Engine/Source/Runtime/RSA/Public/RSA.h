// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Misc/IEngineCrypto.h"
#include "Templates/SharedPointer.h"

struct FRSA
{
	/**
	 * Create a new RSA public/private key from the supplied exponents and modulus binary data. Each of these arrays should contain a single little endian
	 * large integer value
	 */
	static RSA_API FRSAKeyHandle CreateKey(const TArray<uint8>& InPublicExponent, const TArray<uint8>& InPrivateExponent, const TArray<uint8>& InModulus);

	/**
	 * Destroy the supplied key
	 */
	static RSA_API void DestroyKey(const FRSAKeyHandle InKey);

	/**
	 * Returns the size in bytes of the supplied key
	 */
	static RSA_API int32 GetKeySize(const FRSAKeyHandle InKey);

	/**
	 * Returns the maximum number of bytes that can be encrypted in a single payload
	 */
	static RSA_API int32 GetMaxDataSize(const FRSAKeyHandle InKey);

	/**
	 * Encrypt the supplied byte data using the given public key
	 */
	static RSA_API int32 EncryptPublic(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, const FRSAKeyHandle InKey);

	/**
	 * Encrypt the supplied byte data using the given private key
	 */
	static RSA_API int32 EncryptPrivate(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, const FRSAKeyHandle InKey);

	/**
	 * Decrypt the supplied byte data using the given public key
	 */
	static RSA_API int32 DecryptPublic(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, const FRSAKeyHandle InKey);

	/**
	 * Decrypt the supplied byte data using the given private key
	 */
	static RSA_API int32 DecryptPrivate(const TArrayView<const uint8> InSource, TArray<uint8>& OutDestination, const FRSAKeyHandle InKey);
};
