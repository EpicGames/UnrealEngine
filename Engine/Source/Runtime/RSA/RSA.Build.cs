// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RSA : ModuleRules
{
	public RSA(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");

        if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) ||
			Target.Platform == UnrealTargetPlatform.Mac ||
			Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
		{
			PrivateDependencyModuleNames.Add("OpenSSL");
			PrivateDefinitions.Add("RSA_USE_OPENSSL=1");
		}
		else
		{
			PrivateDefinitions.Add("RSA_USE_OPENSSL=0");
		}
	}
}
